{
	"FileVersion" : 3,
	
	"FriendlyName" : "Online Subsystem Amazon",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Access to Amazon platform",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,

    "Modules": [
        {
            "Name": "OnlineSubsystemAmazon",
            "Type": "Runtime",
            "LoadingPhase" : "Default"
        }
    ],
	"LocalizationTargets": [
		{
			"Name": "OnlineSubsystemAmazon",
			"LoadingPolicy": "Always"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		}
	]
}