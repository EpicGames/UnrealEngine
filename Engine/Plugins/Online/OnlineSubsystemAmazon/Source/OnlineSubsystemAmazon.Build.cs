// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class OnlineSubsystemAmazon : ModuleRules
{
	public OnlineSubsystemAmazon(ReadOnlyTargetRules Target) : base(Target)
    {
		PublicDefinitions.Add("ONLINESUBSYSTEMAMAZON_PACKAGE=1");	
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core", 
                "CoreUObject",
				"ApplicationCore",
				"HTTP",
				"Json",
                "OnlineSubsystem", 
			}
		);
	}
}
