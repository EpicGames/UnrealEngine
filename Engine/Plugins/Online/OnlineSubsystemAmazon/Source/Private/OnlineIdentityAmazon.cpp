// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineIdentityAmazon.h"
#include "Misc/ConfigCacheIni.h"
#include "OnlineSubsystemAmazon.h"
#include "OnlineError.h"
#include "HAL/PlatformApplicationMisc.h"

// FUserOnlineAccountAmazon

FUniqueNetIdRef FUserOnlineAccountAmazon::GetUserId() const
{
	return UserIdPtr;
}

FString FUserOnlineAccountAmazon::GetRealName() const
{
	return FString();
}

FString FUserOnlineAccountAmazon::GetDisplayName(const FString& Platform) const
{
	return FString();
}

bool FUserOnlineAccountAmazon::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	return false;
}

bool FUserOnlineAccountAmazon::SetUserAttribute(const FString& AttrName, const FString& AttrValue)
{
	return false;
}

FString FUserOnlineAccountAmazon::GetAccessToken() const
{
	return AuthTicket;
}

bool FUserOnlineAccountAmazon::GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	const FString* FoundAttr = AdditionalAuthData.Find(AttrName);
	if (FoundAttr != NULL)
	{
		OutAttrValue = *FoundAttr;
		return true;
	}
	return false;
}

// FOnlineIdentityAmazon

/**
 * Sets the needed configuration properties
 */
FOnlineIdentityAmazon::FOnlineIdentityAmazon(FOnlineSubsystemAmazon* InSubsystem) :
	Singleton(NULL),
	AmazonSubsystem(InSubsystem),
	LastTickToggle(1),
	LastCheckElapsedTime(0.f),
	TotalCheckElapsedTime(0.f),
	MaxCheckElapsedTime(0.f),
	bHasLoginOutstanding(false),
	LocalUserNumPendingLogin(0)
{
	if (!GConfig->GetString(TEXT("OnlineSubsystemAmazon.OnlineSubsystemAmazon"), TEXT("AmazonEndpoint"), AmazonEndpoint, GEngineIni))
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Missing AmazonEndpoint= in [OnlineSubsystemAmazon.OnlineSubsystemAmazon] of DefaultEngine.ini"));
	}
	if (!GConfig->GetString(TEXT("OnlineSubsystemAmazon.OnlineSubsystemAmazon"), TEXT("RedirectUrl"), RedirectUrl, GEngineIni))
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Missing RedirectUrl= in [OnlineSubsystemAmazon.OnlineSubsystemAmazon] of DefaultEngine.ini"));
	}
	if (!GConfig->GetString(TEXT("OnlineSubsystemAmazon.OnlineSubsystemAmazon"), TEXT("ClientId"), ClientId, GEngineIni))
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Missing AmazonEndpoint= in [OnlineSubsystemAmazon.OnlineSubsystemAmazon] of DefaultEngine.ini"));
	}
	if (!GConfig->GetFloat(TEXT("OnlineSubsystemAmazon.OnlineSubsystemAmazon"), TEXT("RegistrationTimeout"), MaxCheckElapsedTime, GEngineIni))
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Missing RegistrationTimeout= in [OnlineSubsystemAmazon.OnlineSubsystemAmazon] of DefaultEngine.ini"));
		// Default to 30 seconds
		MaxCheckElapsedTime = 30.f;
	}
}

void FOnlineIdentityAmazon::Tick(float DeltaTime, int TickToggle)
{
	// Only tick once per frame
	if (LastTickToggle != TickToggle)
	{
		LastTickToggle = TickToggle;
		TickLogin(DeltaTime);
	}
}

void FOnlineIdentityAmazon::TickLogin(float DeltaTime)
{
	if (bHasLoginOutstanding)
	{
		LastCheckElapsedTime += DeltaTime;
		TotalCheckElapsedTime += DeltaTime;
		// See if enough time has elapsed in order to check for completion
		if (LastCheckElapsedTime > 1.f ||
			// Do one last check if we're getting ready to time out
			TotalCheckElapsedTime > MaxCheckElapsedTime)
		{
			LastCheckElapsedTime = 0.f;
			FString Title;
			if (FPlatformApplicationMisc::GetWindowTitleMatchingText(TEXT("accessToken"), Title))
			{
				bHasLoginOutstanding = false;
				FUserOnlineAccountAmazon User;
				if (ParseLoginResults(Title, User))
				{
					TSharedRef<FUserOnlineAccountAmazon> UserRef(new FUserOnlineAccountAmazon(User.UserId, User.SecretKey, User.AuthTicket));
					UserAccounts.Add(User.UserId, UserRef);					
					TriggerOnLoginCompleteDelegates(LocalUserNumPendingLogin, true, *UserRef->GetUserId(), FString());
				}
				else
				{
					TriggerOnLoginCompleteDelegates(LocalUserNumPendingLogin, false, *FUniqueNetIdAmazon::EmptyId(), FString(TEXT("RegisterUser() failed to parse the user registration results")));
				}
			}
			// Trigger the delegate if we hit the timeout limit
			else if (TotalCheckElapsedTime > MaxCheckElapsedTime)
			{
				bHasLoginOutstanding = false;
				TriggerOnLoginCompleteDelegates(LocalUserNumPendingLogin, false, *FUniqueNetIdAmazon::EmptyId(), FString(TEXT("RegisterUser() timed out without getting the data")));
			}
		}
		// Reset our time trackers if we are done ticking for now
		if (!bHasLoginOutstanding)
		{
			LastCheckElapsedTime = 0.f;
			TotalCheckElapsedTime = 0.f;
		}
	}
}

bool FOnlineIdentityAmazon::ParseLoginResults(const FString& Results,FUserOnlineAccountAmazon& Account)
{
	Account = FUserOnlineAccountAmazon();
	// The format is var=val,var=val,,
	// First split the string into a list of var=val entries
	TArray<FString> Fields;
	Results.ParseIntoArray(Fields, TEXT(","), true);
	// Loop through the fields, setting the proper account field with the data
	for (int Index = 0; Index < Fields.Num(); Index++)
	{
		FString Left;
		FString Right;
		// Split the var=val into left and right strings
		if (Fields[Index].Split(TEXT("="), &Left, &Right))
		{
			// See if this is the account id and set if so
			if (Account.UserId.Len() == 0 && Left == TEXT("amazonCustomerId"))
			{
				Account.UserId = Right;				
			}
			// The access token is used by any calls to the Amazon services
			else if (Account.AuthTicket.Len() == 0 && Left == TEXT("accessToken"))
			{
				Account.AuthTicket = Right;
			}
			// The refresh token is used to generate new access tokens
			else if (Account.SecretKey.Len() == 0 && Left == TEXT("refreshToken"))
			{
				Account.SecretKey = Right;
			}
			// Make sure the server is returning the same state we passed up or we might be having a man in the middle attack
			else if (Left == TEXT("state"))
			{
				if (Right != State)
				{
					Account = FUserOnlineAccountAmazon();
					return false;
				}
			}			
		}
	}
	// keep track of local user mapping to registered ids
	if (Account.UserId.Len() > 0)
	{
		UserIds.Add(LocalUserNumPendingLogin, Account.GetUserId());
	}
	return Account.UserId.Len() && Account.AuthTicket.Len() && Account.SecretKey.Len();
}

TSharedPtr<FUserOnlineAccount> FOnlineIdentityAmazon::GetUserAccount(const FUniqueNetId& UserId) const
{
	TSharedPtr<FUserOnlineAccount> Result;

	const TSharedRef<FUserOnlineAccountAmazon>* FoundUserAccount = UserAccounts.Find(UserId.ToString());
	if (FoundUserAccount != NULL)
	{
		Result = *FoundUserAccount;
	}

	return Result;
}

TArray<TSharedPtr<FUserOnlineAccount> > FOnlineIdentityAmazon::GetAllUserAccounts() const
{
	TArray<TSharedPtr<FUserOnlineAccount> > Result;
	
	for (FUserOnlineAccountAmazonMap::TConstIterator It(UserAccounts); It; ++It)
	{
		Result.Add(It.Value());
	}

	return Result;
}

FUniqueNetIdPtr FOnlineIdentityAmazon::GetUniquePlayerId(int32 LocalUserNum) const
{
	const FUniqueNetIdPtr* FoundId = UserIds.Find(LocalUserNum);
	if (FoundId != NULL)
	{
		return *FoundId;
	}
	return NULL;
}

bool FOnlineIdentityAmazon::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials)
{
	bool bWasSuccessful = false;
	if (!bHasLoginOutstanding && AmazonEndpoint.Len() && RedirectUrl.Len() && ClientId.Len())
	{
		State = FString::FromInt(FMath::Rand() % 100000);
		const FString& Command = FString::Printf(TEXT("%s?scope=profile&response_type=code&redirect_uri=%s&client_id=%s&state=%s"), *AmazonEndpoint, *RedirectUrl, *ClientId, *State);
		// This should open the browser with the command as the URL
		bHasLoginOutstanding = bWasSuccessful = FPlatformMisc::OsExecute(TEXT("open"), *Command);
		if (!bWasSuccessful)
		{
			UE_LOG_ONLINE_IDENTITY(Error, TEXT("RegisterUser() : Failed to execute command %s "), *Command);
		}
		else
		{
			// keep track of local user requesting registration
			LocalUserNumPendingLogin = LocalUserNum;
		}
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Error, TEXT("RegisterUser() : OnlineSubsystemAmazon is improperly configured in DefaultEngine.ini"));
	}
	if (!bWasSuccessful)
	{
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdAmazon::EmptyId(), FString(TEXT("RegisterUser() failed")));
	}
	return bWasSuccessful;
}

FUniqueNetIdPtr FOnlineIdentityAmazon::CreateUniquePlayerId(uint8* Bytes, int32 Size)
{
	if (Bytes != NULL && Size > 0)
	{
		FString StrId = FString::ConstructFromPtrSize((TCHAR*)Bytes, Size);
		return FUniqueNetIdAmazon::Create(StrId);
	}
	return NULL;
}

FUniqueNetIdPtr FOnlineIdentityAmazon::CreateUniquePlayerId(const FString& Str)
{
	return FUniqueNetIdAmazon::Create(Str);
}

// All of the methods below here fail because they aren't supported

bool FOnlineIdentityAmazon::Logout(int32 LocalUserNum)
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		// remove cached user account
		UserAccounts.Remove(UserId->ToString());
		// remove cached user id
		UserIds.Remove(LocalUserNum);
		// not async but should call completion delegate anyway
		TriggerOnLogoutCompleteDelegates(LocalUserNum, true);

		return true;
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("No logged in user found for LocalUserNum=%d."),
			LocalUserNum);
		TriggerOnLogoutCompleteDelegates(LocalUserNum, false);
	}
	return false;
}

bool FOnlineIdentityAmazon::AutoLogin(int32 LocalUserNum)
{
	return false;
}

ELoginStatus::Type FOnlineIdentityAmazon::GetLoginStatus(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		return GetLoginStatus(*UserId);
	}
	return ELoginStatus::NotLoggedIn;
}

ELoginStatus::Type FOnlineIdentityAmazon::GetLoginStatus(const FUniqueNetId& UserId) const 
{
	TSharedPtr<FUserOnlineAccount> UserAccount = GetUserAccount(UserId);
	if (UserAccount.IsValid() &&
		UserAccount->GetUserId()->IsValid() &&
		!UserAccount->GetAccessToken().IsEmpty())
	{
		return ELoginStatus::LoggedIn;
	}
	return ELoginStatus::NotLoggedIn;
}

FString FOnlineIdentityAmazon::GetPlayerNickname(int32 LocalUserNum) const
{
	//@todo - not implemented
	return TEXT("AmazonUser");
}

FString FOnlineIdentityAmazon::GetPlayerNickname(const FUniqueNetId& UserId) const
{
	//@todo - not implemented
	return TEXT("AmazonUser");
}

FString FOnlineIdentityAmazon::GetAuthToken(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		TSharedPtr<FUserOnlineAccount> UserAccount = GetUserAccount(*UserId);
		if (UserAccount.IsValid())
		{
			return UserAccount->GetAccessToken();
		}
	}
	return FString();
}

void FOnlineIdentityAmazon::RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate)
{
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("FOnlineIdentityAmazon::RevokeAuthToken not implemented"));
	FUniqueNetIdRef UserIdRef(UserId.AsShared());
	AmazonSubsystem->ExecuteNextTick([UserIdRef, Delegate]()
	{
		Delegate.ExecuteIfBound(*UserIdRef, FOnlineError(FString(TEXT("RevokeAuthToken not implemented"))));
	});
}

void FOnlineIdentityAmazon::GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI)
{
	Delegate.ExecuteIfBound(UserId, Privilege, (uint32)EPrivilegeResults::NoFailures);
}

FPlatformUserId FOnlineIdentityAmazon::GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const
{
	for (int i = 0; i < MAX_LOCAL_PLAYERS; ++i)
	{
		auto CurrentUniqueId = GetUniquePlayerId(i);
		if (CurrentUniqueId.IsValid() && (*CurrentUniqueId == UniqueNetId))
		{
			return GetPlatformUserIdFromLocalUserNum(i);
		}
	}

	return PLATFORMUSERID_NONE;
}

FString FOnlineIdentityAmazon::GetAuthType() const
{
	return TEXT("");
}
