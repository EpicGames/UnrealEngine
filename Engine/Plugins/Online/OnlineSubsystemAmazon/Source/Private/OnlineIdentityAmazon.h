// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "Interfaces/OnlineIdentityInterface.h"
#include "Serialization/JsonSerializerMacros.h"
#include "OnlineSubsystemNames.h"

class FOnlineSubsystemAmazon;

// from OnlineSubsystemTypes.h
TEMP_UNIQUENETIDSTRING_SUBCLASS(FUniqueNetIdAmazon, AMAZON_SUBSYSTEM);

/**
 * Info associated with an user account generated by this online service
 */
class FUserOnlineAccountAmazon : 
	public FUserOnlineAccount,
	public FJsonSerializable
{
public:

	// FOnlineUser
	
	virtual FUniqueNetIdRef GetUserId() const override;
	virtual FString GetRealName() const override;
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;
	virtual bool SetUserAttribute(const FString& AttrName, const FString& AttrValue) override;

	// FUserOnlineAccount

	virtual FString GetAccessToken() const override;
	virtual bool GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const override;

	// FUserOnlineAccountAmazon

	FUserOnlineAccountAmazon(const FString& InUserId=TEXT(""), const FString& InSecretKey=TEXT(""), const FString& InAuthTicket=TEXT("")) 
		: UserIdPtr(FUniqueNetIdAmazon::Create(InUserId))
		, UserId(InUserId)
		, SecretKey(InSecretKey)
		, AuthTicket(InAuthTicket)
	{ }

	virtual ~FUserOnlineAccountAmazon()
	{
	}

	/** User Id represented as a FUniqueNetId */
	FUniqueNetIdRef UserIdPtr;
	/** Id associated with the user account provided by the online service during registration */
	FString UserId;
	/** Key provided by the online service during registration for future authentication */ 
	FString SecretKey;
	/** Ticket which is provided to user once authenticated by the online service */
	FString AuthTicket;
	/** Any additional data received during registration for use by auth */
	FJsonSerializableKeyValueMap AdditionalAuthData;

	// FJsonSerializable

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("gameAccountId", UserId);
		JSON_SERIALIZE("internalToken", SecretKey);
		JSON_SERIALIZE("authTicket", AuthTicket);
		JSON_SERIALIZE_MAP("additionalAuthData", AdditionalAuthData);
	END_JSON_SERIALIZER
};
/** Mapping from user id to internal online account info (only one per user) */
typedef TMap<FString, TSharedRef<FUserOnlineAccountAmazon> > FUserOnlineAccountAmazonMap;

/**
 * Amazon service implementation of the online identity interface
 */
class FOnlineIdentityAmazon :
	public IOnlineIdentity
{
	/** The endpoint at Amazon we are supposed to hit for auth */
	FString AmazonEndpoint;
	/** The redirect url for Amazon to redirect to upon completion. Note: this is configured at Amazon too */
	FString RedirectUrl;
	/** The client id given to us by Amazon */
	FString ClientId;

	/** Users that have been registered/authenticated */
	FUserOnlineAccountAmazonMap UserAccounts;
	/** Ids mapped to locally registered users */
	TMap<int32, FUniqueNetIdPtr > UserIds;

	/** This interface is a singleton so it can be shared */
	FOnlineIdentityAmazon* Singleton;

	/** Cached pointer to owning subsystem */
	FOnlineSubsystemAmazon* AmazonSubsystem;

	/** Used in case this singleton is shared across subsystems and requires multi-tick protection */
	int LastTickToggle;

	/** The amount of elapsed time since the last check */
	float LastCheckElapsedTime;
	/** Used to determine if we've timed out waiting for the response */
	float TotalCheckElapsedTime;
	/** Config value used to set our timeout period */
	float MaxCheckElapsedTime;
	/** Whether we have a registration in flight or not */
	bool bHasLoginOutstanding;
	/** A value used to verify our response came from our server */
	FString State;
	/** index of local user being registered */
	int32 LocalUserNumPendingLogin;

public:
	// IOnlineIdentity
	
	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;
	virtual bool AutoLogin(int32 LocalUserNum) override;
	virtual TSharedPtr<FUserOnlineAccount> GetUserAccount(const FUniqueNetId& UserId) const override;
	virtual TArray<TSharedPtr<FUserOnlineAccount> > GetAllUserAccounts() const override;
	virtual FUniqueNetIdPtr GetUniquePlayerId(int32 LocalUserNum) const override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(uint8* Bytes, int32 Size) override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(const FString& Str) override;
	virtual ELoginStatus::Type GetLoginStatus(int32 LocalUserNum) const override;
	virtual ELoginStatus::Type GetLoginStatus(const FUniqueNetId& UserId) const override;
	virtual FString GetPlayerNickname(int32 LocalUserNum) const override;
	virtual FString GetPlayerNickname(const FUniqueNetId& UserId) const override;
	virtual FString GetAuthToken(int32 LocalUserNum) const override;
	virtual void RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate) override;
	virtual void GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI=EShowPrivilegeResolveUI::Default) override;
	virtual FPlatformUserId GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const override;
	virtual FString GetAuthType() const override;

	// FOnlineIdentityAmazon

	FOnlineIdentityAmazon(FOnlineSubsystemAmazon* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlineIdentityAmazon()
	{
	}

	/**
	 * Used to do any time based processing of tasks
	 *
	 * @param DeltaTime the amount of time that has elapsed since the last tick
	 * @param TickToggle a toggle so the interface knows if it has been ticked this frame or not
	 */
	void Tick(float DeltaTime, int TickToggle);

	/**
	 * Ticks the registration process handling timeouts, etc.
	 *
	 * @param DeltaTime the amount of time that has elapsed since last tick
	 */
	void TickLogin(float DeltaTime);

	/**
	 * Parses the results into a user account entry
	 *
	 * @param Results the string returned by the login process
	 * @param Account the account structure to fill in
	 *
	 * @return true if it parsed correctly, false otherwise
	 */
	bool ParseLoginResults(const FString& Results, FUserOnlineAccountAmazon& Account);
};

typedef TSharedPtr<FOnlineIdentityAmazon, ESPMode::ThreadSafe> FOnlineIdentityAmazonPtr;
