// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemAmazon.h"
#include "Internationalization/Internationalization.h"
#include "OnlineSubsystemAmazonModule.h"
#include "OnlineIdentityAmazon.h"
#include "Stats/Stats.h"

// FOnlineSubsystemAmazonModule
IMPLEMENT_MODULE(FOnlineSubsystemAmazonModule, OnlineSubsystemAmazon);

/**
 * Class responsible for creating instance(s) of the subsystem
 */
class FOnlineFactoryAmazon : public IOnlineFactory
{
public:

	FOnlineFactoryAmazon() {}
	virtual ~FOnlineFactoryAmazon() {}

	virtual IOnlineSubsystemPtr CreateSubsystem(FName InstanceName)
	{
		FOnlineSubsystemAmazonPtr OnlineSub = MakeShared<FOnlineSubsystemAmazon, ESPMode::ThreadSafe>(InstanceName);
		if (OnlineSub->IsEnabled())
		{
			if(!OnlineSub->Init())
			{
				UE_LOG_ONLINE(Warning, TEXT("Amazon API failed to initialize!"));
				OnlineSub->Shutdown();
				OnlineSub.Reset();
			}
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("Amazon API disabled!"));
			OnlineSub->Shutdown();
			OnlineSub.Reset();
		}

		return OnlineSub;
	}
};

void FOnlineSubsystemAmazonModule::StartupModule()
{
	UE_LOG_ONLINE(Log, TEXT("Amazon Startup!"));

	AmazonFactory = new FOnlineFactoryAmazon();

	// Create and register our singleton factory with the main online subsystem for easy access
	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.RegisterPlatformService(AMAZON_SUBSYSTEM, AmazonFactory);
}

void FOnlineSubsystemAmazonModule::ShutdownModule()
{
	UE_LOG_ONLINE(Log, TEXT("Amazon Shutdown!"));

	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.UnregisterPlatformService(AMAZON_SUBSYSTEM);
	
	delete AmazonFactory;
	AmazonFactory = NULL;
}

IOnlineIdentityPtr FOnlineSubsystemAmazon::GetIdentityInterface() const
{
	return IdentityInterface;
}


bool FOnlineSubsystemAmazon::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemAmazon_Tick);

	if (!FOnlineSubsystemImpl::Tick(DeltaTime))
	{
		return false;
	}

	if (IdentityInterface.IsValid())
	{
		TickToggle ^= 1;
		IdentityInterface->Tick(DeltaTime, TickToggle);
	}
	return true;
}

bool FOnlineSubsystemAmazon::Init()
{
	IdentityInterface = MakeShared<FOnlineIdentityAmazon, ESPMode::ThreadSafe>(this);
	return true;
}

bool FOnlineSubsystemAmazon::Shutdown()
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemAmazon::Shutdown()"));
	IdentityInterface = NULL;
	FOnlineSubsystemImpl::Shutdown();
	return true;
}

FString FOnlineSubsystemAmazon::GetAppId() const
{
	return TEXT("Amazon");
}

bool FOnlineSubsystemAmazon::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) 
{
	if (FOnlineSubsystemImpl::Exec(InWorld, Cmd, Ar))
	{
		return true;
	}
	return false;
}

FText FOnlineSubsystemAmazon::GetOnlineServiceName() const
{
	return NSLOCTEXT("OnlineSubsystemAmazon", "OnlineServiceName", "Amazon");
}

bool FOnlineSubsystemAmazon::IsEnabled() const
{
	// Check the ini for disabling Amazon
	bool bEnableAmazon = FOnlineSubsystemImpl::IsEnabled();
	if (bEnableAmazon)
	{
#if UE_EDITOR
		bEnableAmazon = IsRunningDedicatedServer() || IsRunningGame();
#endif
	}
	return bEnableAmazon;
}

FOnlineSubsystemAmazon::FOnlineSubsystemAmazon(FName InInstanceName) 
	: FOnlineSubsystemImpl(AMAZON_SUBSYSTEM, InInstanceName)
	, IdentityInterface(nullptr)
	, TickToggle(0)
{
}

FOnlineSubsystemAmazon::~FOnlineSubsystemAmazon()
{
}
