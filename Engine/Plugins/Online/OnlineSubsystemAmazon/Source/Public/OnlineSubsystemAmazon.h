// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineSubsystemImpl.h"
#include "OnlineSubsystemAmazonPackage.h"

class FOnlineIdentityAmazon;

/** Forward declarations of all interface classes */
typedef TSharedPtr<class FOnlineIdentityAmazon, ESPMode::ThreadSafe> FOnlineIdentityAmazonPtr;

/**
 * Amazon subsystem
 */
class ONLINESUBSYSTEMAMAZON_API FOnlineSubsystemAmazon :
	public FOnlineSubsystemImpl
{
	class FOnlineFactoryAmazon* AmazonFactory;

	/** Interface to the identity registration/auth services */
	FOnlineIdentityAmazonPtr IdentityInterface;

	/** Used to toggle between 1 and 0 */
	int TickToggle;

PACKAGE_SCOPE:

	/** Only the factory makes instances */
	FOnlineSubsystemAmazon() = delete;
	explicit FOnlineSubsystemAmazon(FName InInstanceName);

public:
	// IOnlineSubsystem

	virtual IOnlineSessionPtr GetSessionInterface() const override { return nullptr; }
	virtual IOnlineFriendsPtr GetFriendsInterface() const override { return nullptr; }
	virtual IOnlinePartyPtr GetPartyInterface() const override { return nullptr; }
	virtual IOnlineGroupsPtr GetGroupsInterface() const override { return nullptr; }
	virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const override { return nullptr; }
	virtual IOnlineUserCloudPtr GetUserCloudInterface() const override { return nullptr; }
	virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const override { return nullptr; }
	virtual IOnlineVoicePtr GetVoiceInterface() const override { return nullptr; }
	virtual IOnlineExternalUIPtr GetExternalUIInterface() const override { return nullptr; }
	virtual IOnlineTimePtr GetTimeInterface() const override { return nullptr; }
	virtual IOnlineTitleFilePtr GetTitleFileInterface() const override { return nullptr; }
	virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override { return nullptr; }
	virtual IOnlineIdentityPtr GetIdentityInterface() const override;
	virtual IOnlineStoreV2Ptr GetStoreV2Interface() const override { return nullptr; }
	virtual IOnlinePurchasePtr GetPurchaseInterface() const override { return nullptr; }
	virtual IOnlineEventsPtr GetEventsInterface() const override { return nullptr; }
	virtual IOnlineAchievementsPtr GetAchievementsInterface() const override { return nullptr; }
	virtual IOnlineSharingPtr GetSharingInterface() const override { return nullptr; }
	virtual IOnlineUserPtr GetUserInterface() const override { return nullptr; }
	virtual IOnlineMessagePtr GetMessageInterface() const override { return nullptr; }
	virtual IOnlinePresencePtr GetPresenceInterface() const override { return nullptr; }
	virtual IOnlineChatPtr GetChatInterface() const override { return nullptr; }
	virtual IOnlineStatsPtr GetStatsInterface() const override { return nullptr; }
	virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const override { return nullptr; }
	virtual IOnlineTournamentPtr GetTournamentInterface() const override { return nullptr; }

	virtual bool Init() override;
	virtual bool Shutdown() override;
	virtual FString GetAppId() const override;
	virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	virtual FText GetOnlineServiceName() const override;
	virtual bool IsEnabled() const override;

	// FTickerBaseObject

	virtual bool Tick(float DeltaTime) override;

	// FOnlineSubsystemAmazon

	/**
	 * Destructor
	 */
	virtual ~FOnlineSubsystemAmazon();
};

typedef TSharedPtr<FOnlineSubsystemAmazon, ESPMode::ThreadSafe> FOnlineSubsystemAmazonPtr;
