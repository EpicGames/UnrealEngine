// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "OnlineSubsystemAmazonPackage.h"

/**
 * Module used for talking with an Amazon service via Http requests
 */
class FOnlineSubsystemAmazonModule :
	public IModuleInterface
{

private:

	/** Class responsible for creating instance(s) of the subsystem */
	class FOnlineFactoryAmazon* AmazonFactory;

public:

	/**
	 * Constructor
	 */
	FOnlineSubsystemAmazonModule() :
		AmazonFactory(NULL)
	{}

	/**
	 * Destructor
	 */
	virtual ~FOnlineSubsystemAmazonModule() 
	{}

	// IModuleInterface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}
};
